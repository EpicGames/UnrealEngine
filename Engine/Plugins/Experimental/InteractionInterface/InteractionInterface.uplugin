{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Interaction Interface",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "InteractableInterface",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "SmartObjects",
			"Enabled": true
		},
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		}
	]
}