// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractableInterfaceLibrary.h"

// Interaction interface module
#include "InteractableTargetInterface.h"
#include "InteractionInterfaceLogs.h"

// Engine
#include "Components/ActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/OverlapResult.h"
#include "GameFramework/Actor.h"

void UInteractableInterfaceLibrary::GetInteractableTargetsFromActor(AActor* Actor, TArray<TScriptInterface<IInteractionTarget>>& OutInteractionTargets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractableInterfaceLibrary::GetInteractableTargetsFromActor);
	
	// If the actor is directly interactable, return that.
	const TScriptInterface<IInteractionTarget> InteractableActor(Actor);
	if (InteractableActor)
	{
		OutInteractionTargets.Add(InteractableActor);
	}

	// If the actor isn't interactable, it might have a component that has a interactable interface.
	TArray<UActorComponent*> InteractableComponents = Actor ? Actor->GetComponentsByInterface(UInteractionTarget::StaticClass()) : TArray<UActorComponent*>();
	for (UActorComponent* InteractableComponent : InteractableComponents)
	{
		OutInteractionTargets.Add(TScriptInterface<IInteractionTarget>(InteractableComponent));
	}
}

void UInteractableInterfaceLibrary::AppendTargetConfiguration(TScriptInterface<IInteractionTarget> Target, const FInteractionContext& Context, FInteractionQueryResults& OutResults)
{
	if (!Target)
	{
		UE_LOGF(LogInteractions, Error, "[%s] Invalid target! Exiting.", __func__);
		return;
	}
	
	Target->AppendTargetConfiguration(Context, OutResults);
}

void UInteractableInterfaceLibrary::BeginInteractionOnTarget(TScriptInterface<IInteractionTarget> Target, const FInteractionContext& Context)
{
	if (!Target)
	{
		UE_LOGF(LogInteractions, Error, "[%s] Invalid target! Exiting.", __func__);
		return;
	}

	Target->BeginInteraction(Context);
}

void UInteractableInterfaceLibrary::ResetQueryResults(UPARAM(ref) FInteractionQueryResults& ToReset)
{
	ToReset.Reset();
}

void UInteractableInterfaceLibrary::AppendInteractableTargetsFromOverlapResults(const TArray<FOverlapResult>& OverlapResults, TArray<TScriptInterface<IInteractionTarget>>& OutInteractableTargets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractableInterfaceLibrary::AppendInteractableTargetsFromOverlapResults);
	
	for (const FOverlapResult& Overlap : OverlapResults)
	{
		const TScriptInterface<IInteractionTarget> InteractableActor(Overlap.GetActor());
		if (InteractableActor)
		{
		 	OutInteractableTargets.AddUnique(InteractableActor);
		}
		
		const TScriptInterface<IInteractionTarget> InteractableComponent(Overlap.GetComponent());
		if (InteractableComponent)
		{
			OutInteractableTargets.AddUnique(InteractableComponent);
		}
	}
}

void UInteractableInterfaceLibrary::AppendInteractableTargetsFromHitResult(const FHitResult& HitResult, TArray<TScriptInterface<IInteractionTarget>>& OutInteractableTargets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractableInterfaceLibrary::AppendInteractableTargetsFromHitResult);
	
	TScriptInterface<IInteractionTarget> InteractableActor(HitResult.GetActor());
	if (InteractableActor)
	{
		OutInteractableTargets.AddUnique(InteractableActor);
	}
	
	TScriptInterface<IInteractionTarget> InteractableComponent(HitResult.GetComponent());
	if (InteractableComponent)
	{
		OutInteractableTargets.AddUnique(InteractableComponent);
	}
}

AActor* UInteractableInterfaceLibrary::GetActorFromInteractableTarget(const TScriptInterface<IInteractionTarget> InteractableTarget)
{
	if (UObject* Object = InteractableTarget.GetObject())
	{
		if (AActor* Actor = Cast<AActor>(Object))
		{
			return Actor;
		}
		else if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object))
		{
			return ActorComponent->GetOwner();
		}
		else
		{
			unimplemented();
		}
	}

	return nullptr;
}
