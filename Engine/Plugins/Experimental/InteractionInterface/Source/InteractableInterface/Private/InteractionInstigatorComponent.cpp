// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractionInstigatorComponent.h"

#include "InteractableTargetInterface.h"
#include "InteractionTypes.h"
#include "InteractionInterfaceLogs.h"

UInteractionInstigatorComponent::UInteractionInstigatorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

void UInteractionInstigatorComponent::AttemptToBeginInteractions(const TArray<TScriptInterface<IInteractionTarget>>& TargetsToInteractWith)
{
	OnAttemptToBeginInteractions(TargetsToInteractWith);
}

void UInteractionInstigatorComponent::OnAttemptToBeginInteractions(const TArray<TScriptInterface<IInteractionTarget>>& TargetsToInteractWith)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractionInstigatorComponent::OnAttemptToBeginInteractions);
	
	UE_LOGF(LogInteractions, Log, "[%s] %ls is attempting to being interaction with targets...", __func__, *GetNameSafe(this));

	FInteractionContextData ContextData = {};
	ContextData.Instigator = this;
	
	// You can make any kind of interaction target data here that you prefer
	FInteractionContext Context = {};

	// The context data about this interaction. Provide some data about your instigator here.
	// Maybe there is a specific smart object slot handle you would like to interact with, or
	// other conditions you would like your target to know about. 
	Context.ContextData = InteractionContextData;

	// Here you could call "IInteractionTarget::AppendTargetConfiguration" if you wanted to on each target.
	// This would allow you to begin interactions conditionally based on some criteria that you set up for your 
	// game if desired.

	// Get info about how we are supposed to respond to any interactions and metadata about them
	for (const TScriptInterface<IInteractionTarget>& Target : TargetsToInteractWith)
	{
		Context.Target = Target;
		Target->BeginInteraction(Context);
	}
}
