// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractionInterfaceModule.h"

#include "Modules/ModuleManager.h"

FInteractionInterfaceModule& FInteractionInterfaceModule::Get()
{
	static FInteractionInterfaceModule& Singleton = FModuleManager::LoadModuleChecked<FInteractionInterfaceModule>("InteractionInterface");
	return Singleton;
}

bool FInteractionInterfaceModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("InteractionInterface");
}

IMPLEMENT_MODULE(FInteractionInterfaceModule, InteractableInterface)