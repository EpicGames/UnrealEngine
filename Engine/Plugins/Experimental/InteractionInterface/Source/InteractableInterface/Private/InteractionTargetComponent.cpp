// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractionTargetComponent.h"
#include "InteractionTypes.h"
#include "InteractionInterfaceLogs.h"

UInteractionTargetComponent::UInteractionTargetComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UInteractionTargetComponent::BP_AppendTargetConfiguration(const FInteractionContext& Context, FInteractionQueryResults& OutResults) const
{
	// Just call in the interface implementation on this blueprint.
	AppendTargetConfiguration(Context, OutResults);
}

void UInteractionTargetComponent::BP_BeginInteraction(const FInteractionContext& Context)
{
	// Just call in the interface implementation on this blueprint.
	BeginInteraction(Context);
}

void UInteractionTargetComponent::AppendTargetConfiguration(const FInteractionContext& QueryContext, FInteractionQueryResults& OutResults) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractionTargetComponent::AppendTargetConfiguration);
	
	// Add any info about the interactions that are available on this target.
	OutResults.AvailableInteractions.Append(TargetConfigs);
}

void UInteractionTargetComponent::BeginInteraction(const FInteractionContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInteractionTargetComponent::BeginInteraction);
	
	UE_LOGF(LogInteractions, Log, "[%s] Native C++ impl of begin interaction on this target %ls", __func__, *GetNameSafe(this));

	// Broadcast BP event that you can bind to
	OnBeginInteractionCallback.Broadcast(Context);
}
