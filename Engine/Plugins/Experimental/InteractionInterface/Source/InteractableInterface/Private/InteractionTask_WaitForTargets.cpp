// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractionTask_WaitForTargets.h"

// Engine
#include "AbilitySystemComponent.h"
#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"
#include "TimerManager.h"
#include "DrawDebugHelpers.h"


// Interactable Interface
#include "InteractableInterfaceLibrary.h"
#include "InteractionInterfaceLogs.h"

//////////////////////////////////////////////////////////
// UAbilityTask_GrantNearbyInteractionData

UAbilityTask_GrantNearbyInteractionData* UAbilityTask_GrantNearbyInteractionData::GrantAbilitiesForNearbyInteractionData(
	UGameplayAbility* OwningAbility,
	ECollisionChannel TraceChannel,
	float InteractionScanRange /* = 500.f */,
	float InteractionScanRate /* = 0.1f */)
{
	UAbilityTask_GrantNearbyInteractionData* MyObj = NewAbilityTask<UAbilityTask_GrantNearbyInteractionData>(OwningAbility);
	MyObj->InteractionScanRange = InteractionScanRange;
	MyObj->InteractionScanRate = InteractionScanRate;
	MyObj->InteractionTraceChannel = TraceChannel;
	
	return MyObj;
}

void UAbilityTask_GrantNearbyInteractionData::Activate()
{
	// Start scanning for nearby targets at our scan rate
	SetWaitingOnAvatar();
	
	UWorld* World = GetWorld();
	if (!ensure(World))
	{
		return;
	}

	// Start a time for the scan rate of when the gather interactable target data
	World->GetTimerManager().SetTimer(
		QueryTimerHandle,
		this,
		&UAbilityTask_GrantNearbyInteractionData::QueryNearbyInteractables,
		InteractionScanRate,
		/* bLoop= */ true);
}

void UAbilityTask_GrantNearbyInteractionData::OnDestroy(bool AbilityEnded)
{
	// Clear the scan timer
	if (UWorld* World = GetWorld())
	{
		World->GetTimerManager().ClearTimer(QueryTimerHandle);
	}

	Super::OnDestroy(AbilityEnded);
}

void UAbilityTask_GrantNearbyInteractionData::QueryNearbyInteractables()
{
	UWorld* World = GetWorld();
	AActor* ActorOwner = GetAvatarActor();

	if (!World || !ActorOwner)
	{
		return;
	}

	// TODO: Make this an Async overlap check
	// FTraceHandle handle = World->AsyncOverlapByChannel
	// if ! handle valid... do another trace.
	// Leaving this as a TODO to reduce complexity and make sure this stuff really works well before optimizing
	
	// Do a sphere trace around the requesting actor and gather the available interactions around them
	FCollisionQueryParams Params(SCENE_QUERY_STAT(UAbilityTask_GrantNearbyInteractionData), false);

	// Ignore the owner of this ability...
	Params.AddIgnoredActor(ActorOwner);

	TArray<FOverlapResult> OverlapResults;

	World->OverlapMultiByChannel(
		OUT OverlapResults,
		ActorOwner->GetActorLocation(),
		FQuat::Identity,
		InteractionTraceChannel,
		FCollisionShape::MakeSphere(InteractionScanRange),
		Params);
		
#if ENABLE_DRAW_DEBUG
	DrawDebugSphere(World, ActorOwner->GetActorLocation(), InteractionScanRange, 10, FColor::Green, false, InteractionScanRate);
#endif
	
	if (OverlapResults.IsEmpty())
	{
		return;
	}

	// Gather the IInteractionTarget interface from our query result
	TArray<TScriptInterface<IInteractionTarget>> InteractableTargets;
	UInteractableInterfaceLibrary::AppendInteractableTargetsFromOverlapResults(OverlapResults, OUT InteractableTargets);

	// TODO: We could make a smart object query here to gather available smart objects in the given area
	// or make a subclass of this task which does a query using smart object world conditions

	// If the available targets from the spatial query has changed, then broadcast our delegate that
	// there has been a change in what targets are available
	if (InteractableTargets != CurrentAvailableTargets)
	{
		CurrentAvailableTargets = InteractableTargets;
		OnAvailableInteractionTargetsChanged.Broadcast(InteractableTargets);
	}
}

//////////////////////////////////////////////////////////
// UGameplayAbility_Interact 
UGameplayAbility_Interact::UGameplayAbility_Interact(const FObjectInitializer& ObjectInitializer)
{
	InstancingPolicy = EGameplayAbilityInstancingPolicy::InstancedPerActor;
	NetExecutionPolicy = EGameplayAbilityNetExecutionPolicy::LocalPredicted;
}

void UGameplayAbility_Interact::ActivateAbility(const FGameplayAbilitySpecHandle Handle, const FGameplayAbilityActorInfo* ActorInfo, const FGameplayAbilityActivationInfo ActivationInfo, const FGameplayEventData* TriggerEventData)
{
	Super::ActivateAbility(Handle, ActorInfo, ActivationInfo, TriggerEventData);

	const UAbilitySystemComponent* AbilitySystem = GetAbilitySystemComponentFromActorInfo();

	// Create a task which will check for nearby interaction targets! 
	if (AbilitySystem && AbilitySystem->GetOwnerRole() == ROLE_Authority)
	{
		UAbilityTask_GrantNearbyInteractionData* Task = UAbilityTask_GrantNearbyInteractionData::GrantAbilitiesForNearbyInteractionData(
			this,
			InteractionTraceChannel,
			InteractionScanRange,
			InteractionScanRate);

		Task->OnAvailableInteractionTargetsChanged.AddUniqueDynamic(this, &UGameplayAbility_Interact::HandleTargetsUpdatedFromTask);
		
		Task->ReadyForActivation();
	}
}

void UGameplayAbility_Interact::HandleTargetsUpdatedFromTask(const TArray<TScriptInterface<IInteractionTarget>>& AvailableTargets)
{
	UpdateInteractions(AvailableTargets);	
}

void UGameplayAbility_Interact::OnAvailableInteractionsUpdated_Implementation()
{
	UE_LOGF(LogInteractions, Log, "[%s] %d targets are now available to be interacted with.", __func__, CurrentAvailableTargets.Num());
}

void UGameplayAbility_Interact::OnTriggerInteraction_Implementation()
{
	UE_LOGF(LogInteractions, Log, "[%s] trigger interactions on %d targets", __func__, CurrentAvailableTargets.Num());
}

void UGameplayAbility_Interact::UpdateInteractions(const TArray<TScriptInterface<IInteractionTarget>>& AvailableTargets)
{
	CurrentAvailableTargets = AvailableTargets;

	// Notify BP/C++ that the available interactions have changed
	OnAvailableInteractionsUpdated();
}

void UGameplayAbility_Interact::TriggerInteraction()
{
	// By default for now just let the BP implementations or C++ subclasses of this ability
	// decide how or which targets to interact with.
	OnTriggerInteraction();
}