// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "InteractableInstigator.generated.h"

class IInteractionTarget;

// This class does not need to be modified.
UINTERFACE(meta=(CannotImplementInterfaceInBlueprint))
class INTERACTABLEINTERFACE_API UInteractableInstigator : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that represents a single "Instigator" of an interaction.
 * 
 * Instigators are objects which are allowed to interact with targets. They
 * can attempt to begin interacting with a given set of targets.
 */
class INTERACTABLEINTERFACE_API IInteractableInstigator
{
	GENERATED_BODY()
	
protected:
	
	virtual void OnAttemptToBeginInteractions(const TArray<TScriptInterface<IInteractionTarget>>& TargetsToInteractWith) = 0;
};