// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "InteractionTypes.h"

#include "InteractableInterfaceLibrary.generated.h"

class AActor;
class IInteractionTarget;
struct FHitResult;
struct FOverlapResult;

/**
 * BP function helpers to utilize the Interactable Interface.
 */
UCLASS()
class INTERACTABLEINTERFACE_API UInteractableInterfaceLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Populates the given OutInteractionTargets array with all IInteractionTarget interface
	 * objects on the given actor.
	 *
	 * @param Actor						The actor to check for targets on
	 * @param OutInteractionTargets		Array of pointers to any interaction targets on the given actor
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions")
	static void GetInteractableTargetsFromActor(AActor* Actor, TArray<TScriptInterface<IInteractionTarget>>& OutInteractionTargets);

	/**
	 * Determines what the configuration of this target is. 
	 * Gather information about this specific target so that it can be displayed
	 * to the player and provide access to what behavior should occur in response
	 * to this interaction.
	 *
	 * @param Target		The interaction target to get the configuration of
	 * @param Context		The context of this interaction.
	 * @param OutResults	Output results that this target will populate
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions", meta=(AutoCreateRefTerm="Context"))
	static void AppendTargetConfiguration(TScriptInterface<IInteractionTarget> Target, const FInteractionContext& Context, UPARAM(ref) FInteractionQueryResults& OutResults);

	/**
	 * Called when this target is interacted with. Implement any state changes or gameplay affects
	 * you want this interaction to have here
	 *
	 * @param Target		The interaction target to begin interacting with
	 * @param Context		The context of this interaction. 
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions", meta=(AutoCreateRefTerm="Context"))
	static void BeginInteractionOnTarget(TScriptInterface<IInteractionTarget> Target, const FInteractionContext& Context);
	
	/**
	* Resets the values of the given interaction query results to be empty.
	*/
	UFUNCTION(BlueprintCallable, Category="Interactions")
	static void ResetQueryResults(UPARAM(ref) FInteractionQueryResults& ToReset);

	/**
	 * Given a set of overlap results, append any found IInteractionTargets to the OutInteractableTargets array.
	 *
	 * @param OverlapResults			The overlap results from a physics query to process.
	 * @param OutInteractableTargets	The array of targets to append any found targets to. This array will not be reset.
	 */
	static void AppendInteractableTargetsFromOverlapResults(const TArray<FOverlapResult>& OverlapResults, TArray<TScriptInterface<IInteractionTarget>>& OutInteractableTargets);

	/**
	 * Given a hit result, append any found IInteractionTargets to the OutInteractableTargets array.
	 *
	 * @param HitResult					The hit result to process for targets
	 * @param OutInteractableTargets	The array of targets to append any found targets to. This array will not be reset.
	 */
	static void AppendInteractableTargetsFromHitResult(const FHitResult& HitResult, TArray<TScriptInterface<IInteractionTarget>>& OutInteractableTargets);

	static AActor* GetActorFromInteractableTarget(const TScriptInterface<IInteractionTarget> InteractableTarget);
};