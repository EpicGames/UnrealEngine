// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "InteractableTargetInterface.generated.h"

struct FInteractionContext;
struct FInteractionQueryResults;

// This class does not need to be modified.
UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class INTERACTABLEINTERFACE_API UInteractionTarget : public UInterface
{
	GENERATED_BODY()
};

/**
 * Implement this interface on things that are able to be interacted with.
 *
 * A target should be something in the world that can be interacted with. For example,
 * a treasure chest which can be opened, an item that can picked up, or some narrative design
 * element for your game.
 */
class INTERACTABLEINTERFACE_API IInteractionTarget
{
	GENERATED_BODY()
public:
	
	/**
	 * Determines what the configuration of this target is. 
	 * Gather information about this specific target so that it can be displayed
	 * to the player and provide access to what behavior should occur in response
	 * to this interaction.
	 *
	 * @param Context		The context of this interaction.
	 * @param OutResults	Output results that this target will populate
	 */
	virtual void AppendTargetConfiguration(const FInteractionContext& Context, OUT FInteractionQueryResults& OutResults) const = 0;

	/**
	 * Called when this target is interacted with. Implement any state changes or gameplay affects
	 * you want this interaction to have here
	 *
	 * @param Context		The context of this interaction. This is customizable for your
	 *						game by adding additional context types.
	 */
	virtual void BeginInteraction(const FInteractionContext& Context) = 0;	
};