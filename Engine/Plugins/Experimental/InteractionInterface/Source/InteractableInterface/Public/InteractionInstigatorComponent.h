// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractableInstigator.h"
#include "Components/ActorComponent.h"
#include "StructUtils/InstancedStruct.h"

#include "InteractionInstigatorComponent.generated.h"

class IInteractionTarget;

UCLASS(Blueprintable, ClassGroup = Gameplay, meta = (BlueprintSpawnableComponent), HideCategories = (Activation, AssetUserData, Collision))
class INTERACTABLEINTERFACE_API UInteractionInstigatorComponent :
	public UActorComponent,
	public IInteractableInstigator
{
	GENERATED_BODY()
	
public:

	UInteractionInstigatorComponent(const FObjectInitializer& ObjectInitializer);

	/**
	 * Attempts to begin interacting with the given array of targets
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions")
	void AttemptToBeginInteractions(const TArray<TScriptInterface<IInteractionTarget>>& TargetsToInteractWith);
	
protected:

	/**
	* Data about this specific interaction
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Interactions", meta = (BaseStruct = "/Script/InteractableInterface.InteractionContextData"))
	FInstancedStruct InteractionContextData;

	//~Begin IInteractableInstigator interface
	virtual void OnAttemptToBeginInteractions(const TArray<TScriptInterface<IInteractionTarget>>& TargetsToInteractWith) override;
	//~End IInteractableInstigator interface
};