// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

INTERACTABLEINTERFACE_API DECLARE_LOG_CATEGORY_EXTERN(LogInteractions, Log, All);