// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractableTargetInterface.h"
#include "Components/ActorComponent.h"
#include "Components/BoxComponent.h"
#include "StructUtils/InstancedStruct.h"

#include "InteractionTargetComponent.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FBeginInteractionCallback, const FInteractionContext&, Context);

/**
 * A target that can be interacted with by an interaction instigator.
 * 
 * Add this component to actors that you would like to be intractable.
 */
UCLASS(Blueprintable, ClassGroup = Gameplay, meta = (BlueprintSpawnableComponent), HideCategories = (Activation, AssetUserData))
class INTERACTABLEINTERFACE_API UInteractionTargetComponent :
	public UBoxComponent,
	public IInteractionTarget
{
	GENERATED_BODY()
public:

	UInteractionTargetComponent(const FObjectInitializer& ObjectInitializer);

	// A callback for when this target begins interaction.
	UPROPERTY(BlueprintAssignable, Category="Interactions")
	FBeginInteractionCallback OnBeginInteractionCallback;
	
	/**
     * Determines what the configuration of this target is. 
     * Gather information about this specific target so that it can be displayed
     * to the player and provide access to what behavior should occur in response
     * to this interaction.
     *
     * @param Context		The context of this interaction.
     * @param OutResults	Output results that this target will populate
     */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="Interactions", meta=(DisplayName="Get Target Configuration", AutoCreateRefTerm="Context,OutResults"))
	void BP_AppendTargetConfiguration(const FInteractionContext& Context, FInteractionQueryResults& OutResults) const;
	
	/**
	 * Called when this target is interacted with. Implement any state changes or gameplay affects
	 * you want this interaction to have here
	 *
	 * @param Context		The context of this interaction. This is customizable for your
	 *						game by adding additional context types.
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions", meta=(DisplayName="Begin Interaction",AutoCreateRefTerm="Context"))
	void BP_BeginInteraction(const FInteractionContext& Context);
	
protected:

	//~ Begin IInteractionTarget interface
	virtual void AppendTargetConfiguration(const FInteractionContext& QueryContext, FInteractionQueryResults& OutResults) const override;
	virtual void BeginInteraction(const FInteractionContext& Context) override;
	//~ End of IInteractionTarget interface

	/**
	 * The configuration for this target component.
	 *
	 * Stores metadata about this interaction can be used to build UI or make decisions about
	 * which target is currently desired the most by the instigator.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Interaction", meta = (BaseStruct = "/Script/InteractableInterface.InteractionTargetConfiguration"))
	TArray<FInstancedStruct> TargetConfigs;
};

