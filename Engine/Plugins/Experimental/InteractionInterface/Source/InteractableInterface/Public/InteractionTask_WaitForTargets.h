// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Engine
#include "CoreMinimal.h"
#include "Abilities/GameplayAbility.h"
#include "Abilities/Tasks/AbilityTask.h"
#include "Engine/EngineTypes.h"

// Interaction Interface
#include "InteractionTypes.h"

#include "InteractionTask_WaitForTargets.generated.h"

class UIndicatorDescriptor;
class UObject;
class UUserWidget;
struct FFrame;
struct FGameplayAbilityActorInfo;
struct FGameplayEventData;

/**
 * Delegate fired for when the nearby available interactable targets have changed.
 */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FAvailableInteractionTargetsChanged, const TArray<TScriptInterface<IInteractionTarget>>&, InteractableOptions);

/**
 * Gameplay Ability Task that will query all available interaction targets at a given
 * interval around it's owning ability actor.
 *
 * This task will gather nearby interactive targets and make them available to you
 * via blueprint/C++ with the OnAvailableInteractionTargetsChanged delegate
 */
UCLASS()
class INTERACTABLEINTERFACE_API UAbilityTask_GrantNearbyInteractionData : public UAbilityTask
{
	GENERATED_BODY()

public:
	/**
	 * Wait until an overlap occurs. This will need to be better fleshed out so we can specify game specific collision requirements
	 */
	UFUNCTION(BlueprintCallable, Category="Ability|Tasks", meta = (HidePin = "OwningAbility", DefaultToSelf = "OwningAbility", BlueprintInternalUseOnly = "TRUE"))
	static UAbilityTask_GrantNearbyInteractionData* GrantAbilitiesForNearbyInteractionData(
		UGameplayAbility* OwningAbility,
		ECollisionChannel TraceChannel,
		float InteractionScanRange = 500.0f,
		float InteractionScanRate = 0.1f);

	/**
	 * Delegate fired when the available interaction targets near the owner of this ability task have changed
	 */
	UPROPERTY(BlueprintAssignable)
	FAvailableInteractionTargetsChanged OnAvailableInteractionTargetsChanged;

protected:

	//~ UAbilityTask interface
	virtual void Activate() override;
	virtual void OnDestroy(bool AbilityEnded) override;
	//~ End of UAbilityTask interface

	/**
	 * Runs a query to gather the nearby interactable objects to this ability's owner.
	 * 
	 */
	virtual void QueryNearbyInteractables();

	/**
	 * How often to scan for targets. A world OverlapMultiByChannel call will happen
	 * at this rate to check for available interactions around us.
	 */
	float InteractionScanRange = 500.0f;

	/**
	 * How often to scan for targets. A world OverlapMultiByChannel call will happen
	 * at this rate to check for available interactions around us.
	 */
	float InteractionScanRate = 0.1f;

	/**
	 * The collision channel to check for interactable targets on
	 */
	ECollisionChannel InteractionTraceChannel = ECC_GameTraceChannel1;

	/**
	 * Timer handle that is populated on this task's Activate function and cleared OnDestroy
	 * for how often to Query for targets.
	 */
	FTimerHandle QueryTimerHandle;

	/**
	 * Array of interaction targets which are in range from the most recent query
	 */
	UPROPERTY()
	TArray<TScriptInterface<IInteractionTarget>> CurrentAvailableTargets;
};



UINTERFACE()
class INTERACTABLEINTERFACE_API UInteractionAbilityInterface : public UInterface
{
	GENERATED_BODY()
};


class INTERACTABLEINTERFACE_API IInteractionAbilityInterface
{
	GENERATED_BODY()
protected:

	/**
	 * Called when this ability's available interaction targets have been updated.
	 * 
	 * This is a good place to update some UI or display some message to the user
	 * that they can now interact with the current targets.
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Interactions")
	void OnAvailableInteractionsUpdated();
	virtual void OnAvailableInteractionsUpdated_Implementation() {}
	
	/**
     * Triggers the interaction with one or more of the currently available targets.
     * Override this in blueprints or native C++ to decide which of the currently available targets
     * you would like to interact with and how.
     */
    UFUNCTION(BlueprintNativeEvent, Category="Interactions")
    void OnTriggerInteraction();
    virtual void OnTriggerInteraction_Implementation() {}
};


/**
 * Gameplay ability for interacting with a target(s).
 *
 * This ability will trigger interactions on its current list of available targets
 * which are populated via the UpdateInteractions functions.
 *
 * When UpdateInteractions is called, it provides a nice place to update
 * some UI or other things you may want to do to display to your player
 * that interactions are now available.
 */
UCLASS(Abstract)
class INTERACTABLEINTERFACE_API UGameplayAbility_Interact
	: public UGameplayAbility,
	public IInteractionAbilityInterface
{
	GENERATED_BODY()

public:

	UGameplayAbility_Interact(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/**
	 * Update the available interactions that this ability can trigger.
	 * This is normally populated via an async task running in the ability blueprint
	 * to gather nearby targets. 
	 */
	UFUNCTION(BlueprintCallable, Category="Interactions")
	void UpdateInteractions(const TArray<TScriptInterface<IInteractionTarget>>& AvailableTargets);

	/**
	 * Attempts to begin the interaction with the current targets.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interactions")
	void TriggerInteraction();

protected:

	//~ UGameplayAbility
	virtual void ActivateAbility(
		const FGameplayAbilitySpecHandle Handle,
		const FGameplayAbilityActorInfo* ActorInfo,
		const FGameplayAbilityActivationInfo ActivationInfo,
		const FGameplayEventData* TriggerEventData)
	override;
	//~ End of UGameplayAbility

	void HandleTargetsUpdatedFromTask(const TArray<TScriptInterface<IInteractionTarget>>& AvailableTargets);

	/**
	 * Native C++ implementation of OnAvailableInteractionsUpdated.
	 * Override this to do any work you may need to when new interactions become available
	 */
	virtual void OnAvailableInteractionsUpdated_Implementation() override;
	virtual void OnTriggerInteraction_Implementation() override;
	
	/**
	 * How often to scan for targets. A world OverlapMultiByChannel call will happen
	 * at this rate to check for available interactions around us.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Interactions")
	float InteractionScanRate = 0.1f;

	/**
	 * The range to scan for available targets. A sphere of this radius will be cast around this ability's
	 * owning actor to check for nearby interactions.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Interactions")
	float InteractionScanRange = 500.0f;

	/**
	 * The collision channel to use when checking for interaction targets within the given area
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Interactions")
	TEnumAsByte<ECollisionChannel> InteractionTraceChannel = ECC_GameTraceChannel1;

	/**
	 * Array of available interaction targets to interact with. This is populated by "UpdateInteractions"
	 * and normally after an ability task to gather the available targets has completed.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Interactions")
	TArray<TScriptInterface<IInteractionTarget>> CurrentAvailableTargets;
};