// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTagContainer.h"

#include "InteractionTypes.generated.h"

class IInteractionTarget;
class IInteractableInstigator;
class UInteractionResponse;

/**
 * Contains data about a single interactable target. This is information that you may want to use
 * to build some UI or decide on what the state of certain interactable objects is.
 */
USTRUCT(BlueprintType)
struct FInteractionTargetConfiguration
{
	GENERATED_BODY()
	
public:
	
	/**
	 * The display name that can be used for this interaction
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interactions")
	FText DisplayName;

	// Any other metadata that we may want for a single interaction
	// Maybe some kind of widget here, or we make a subclass of FInteractionTargetConfiguration that supports that
	// Games can extend this type to add whatever data they need here.
};

/**
 * Data about a specific interaction that is happening.
 *
 * It is encouraged that you make structs that inherit from this one to add
 * custom game logic and conditions you may need for any given interaction.
 * For example, you may have a target which is only able to be used if the
 * player has a specific item equipped, or some other conditional state like that.
 */
USTRUCT(BlueprintType)
struct FInteractionContextData
{
	GENERATED_BODY()

	/**
	 * The interaction instigator who is doing the querying!
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="General")
	TScriptInterface<IInteractableInstigator> Instigator;

	/**
	 * Some tags that are unique to this interaction and can be used to provide some context
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="General")
	FGameplayTagContainer InteractionTags;
};

/**
 * The context of a given interaction query. This holds some state about
 * what is currently querying for intractable objects and will allow you
 * to specify some specific game state around each interaction.
 *
 * @see IInteractionTarget::BeginInteraction
 */
USTRUCT(BlueprintType)
struct FInteractionContext
{
	GENERATED_BODY()
public:

	/**
	 * The interactable target that should be used
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category=General)
	TScriptInterface<IInteractionTarget> Target;
	
	/**
	 * Data about this specific interaction query
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (BaseStruct = "/Script/InteractableInterface.InteractionContextData"))
	FInstancedStruct ContextData;
};

/**
 * A struct that will be populated with all the available interaction options for a given target.
 *
 * These query results are populated by Interaction Targets. Each target may have
 * multiple configurations on it (i.e. a bench with two seats, or a car with 4 doors to open)
 *
 * @see IInteractionTarget::GetTargetConfiguration
 */
USTRUCT(BlueprintType)
struct FInteractionQueryResults
{
	GENERATED_BODY()
public:	

	/**
	* Resets the values of this query results to be empty.
	*/
	void Reset()
	{
		AvailableInteractions.Reset();
	}

	/**
	 * Array of available interactions that can be started.
	 * Add to this array for any interaction which you would like to be presented
	 * as an available option in response to this query.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, meta = (BaseStruct = "/Script/InteractableInterface.InteractionTargetConfiguration"))
	TArray<FInstancedStruct> AvailableInteractions;
};
